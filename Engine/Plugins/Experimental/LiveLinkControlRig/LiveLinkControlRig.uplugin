{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Live Link Control Rig",
	"Description": "Allows access to LiveLink Data through Control Rig",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"Plugins": [
		{
			"Name": "ControlRig",
			"Enabled":  true
		}
	],
	"Modules": [
		{
			"Name": "LiveLinkControlRig",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}