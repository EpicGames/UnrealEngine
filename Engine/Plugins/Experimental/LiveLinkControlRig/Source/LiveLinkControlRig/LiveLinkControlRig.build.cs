// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class LiveLinkControlRig : ModuleRules
	{
		public LiveLinkControlRig(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"LiveLinkInterface",
			});

			PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"InputCore",
				"Media",
				"Projects",
				"SlateCore",
				"Slate",
				"RigVM",
				"ControlRig",
			});
		}
	}
}
