// Copyright Epic Games, Inc. All Rights Reserved.
#include "LiveLinkControlRig.h"

IMPLEMENT_MODULE(FLiveLinkControlRigModule, LiveLinkControlRig)

void FLiveLinkControlRigModule::StartupModule()
{

}

void FLiveLinkControlRigModule::ShutdownModule()
{

}

