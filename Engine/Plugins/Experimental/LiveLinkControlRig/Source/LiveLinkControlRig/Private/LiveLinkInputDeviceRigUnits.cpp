// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkInputDeviceRigUnits.h"

#include "Features/IModularFeatures.h"
#include "Roles/LiveLinkInputDeviceRole.h"

#include "ILiveLinkClient.h"
#include "Roles/LiveLinkInputDeviceTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkInputDeviceRigUnits)

namespace LiveLinkInputDeviceRigUtils
{

ILiveLinkClient* TryGetLiveLinkClient()
{
	static ILiveLinkClient* LiveLinkClient = []() -> ILiveLinkClient*
	{
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
		{
			return static_cast<ILiveLinkClient*>(&ModularFeatures.GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName));
		}
		return nullptr;
	}();
	return LiveLinkClient;
}

}

FRigUnit_LiveLinkEvaluateInputDeviceValue_Execute()
{
	// Get value by property name from input device gamepad.
	if (ILiveLinkClient* LiveLinkClient = LiveLinkInputDeviceRigUtils::TryGetLiveLinkClient())
	{
		FLiveLinkSubjectFrameData FrameData;
		const bool bHaveFrameData =	LiveLinkClient->EvaluateFrame_AnyThread(SubjectName, ULiveLinkInputDeviceRole::StaticClass(),FrameData);
		if (bHaveFrameData)
		{
			if (FLiveLinkGamepadInputDeviceFrameData* Data = FrameData.FrameData.Cast<FLiveLinkGamepadInputDeviceFrameData>())
			{
				InputDeviceData = *Data;
			}
		}
	}
}
