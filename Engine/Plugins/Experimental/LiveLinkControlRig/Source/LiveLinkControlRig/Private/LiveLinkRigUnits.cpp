// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkRigUnits.h"
#include "Units/RigUnitContext.h"
#include "Features/IModularFeatures.h"
#include "ILiveLinkClient.h"
#include "Roles/LiveLinkAnimationRole.h"
#include "Roles/LiveLinkTransformRole.h"
#include "Roles/LiveLinkTransformTypes.h"
#include "Roles/LiveLinkBasicRole.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkRigUnits)

ILiveLinkClient* LiveLinkControlRigUtilities::TryGetLiveLinkClient()
{
	static ILiveLinkClient* LiveLinkClient = nullptr;

	if (LiveLinkClient == nullptr)
	{
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
		{
			LiveLinkClient = &ModularFeatures.GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
		}
	}
	return LiveLinkClient;
}

FRigUnit_LiveLinkEvaluteFrameAnimation_Execute()
{
	// Ensure a default value in case the evaluation fails
	SubjectFrame = FSubjectFrameHandle();

	// Try to update SubjectFrame from Live Link
	if (ILiveLinkClient* LiveLinkClient = LiveLinkControlRigUtilities::TryGetLiveLinkClient())
	{
		FLiveLinkSubjectFrameData FrameData;
		if (LiveLinkClient->EvaluateFrame_AnyThread(SubjectName, ULiveLinkAnimationRole::StaticClass(), FrameData))
		{
			FLiveLinkBlueprintDataStruct BlueprintStruct(FSubjectFrameHandle::StaticStruct(), &SubjectFrame);
			GetDefault<ULiveLinkAnimationRole>()->InitializeBlueprintData(FrameData, BlueprintStruct);
		}
	}

	// Debug Drawing if required
	if (bDrawDebug && ExecuteContext.GetDrawInterface() != nullptr)
	{
		/*
		FRigBoneHierarchy DebugHierarchy;

		for (int BoneIdx = 0; BoneIdx < SubjectFrame.GetNumberOfTransforms(); ++BoneIdx)
		{
			FLiveLinkTransform CurrentTransform;
			SubjectFrame.GetTransformByIndex(BoneIdx, CurrentTransform);
			FName CurrentBoneName;
			CurrentTransform.GetName(CurrentBoneName);
			FName CurrentParentName(NAME_None);
			if (CurrentTransform.HasParent())
			{
				FLiveLinkTransform ParentTransform;
				CurrentTransform.GetParent(ParentTransform);
				ParentTransform.GetName(CurrentParentName);
			}
			FTransform LocalSpaceTransform, GlobalSpaceTransform;
			CurrentTransform.GetTransformParentSpace(LocalSpaceTransform);
			CurrentTransform.GetTransformRootSpace(GlobalSpaceTransform);
			DebugHierarchy.Add(CurrentBoneName, CurrentParentName, ERigBoneType::User, LocalSpaceTransform, LocalSpaceTransform, GlobalSpaceTransform);
		}
		Context.DrawInterface->DrawHierarchy(DebugDrawOffset, DebugHierarchy, EControlRigDrawHierarchyMode::Axes, 1.0f, DebugColor, 1.0f);
		*/
	}
}

FRigUnit_LiveLinkGetTransformByName_Execute()
{
	// Ensure a default value in case the evaluation fails
	Transform = FTransform::Identity;

	// Try to update Transform from the Subject Frame
	FLiveLinkTransform LiveLinkTransform;
	SubjectFrame.GetTransformByName(TransformName, LiveLinkTransform);
	switch (Space)
	{
	case ERigVMTransformSpace::GlobalSpace:
		LiveLinkTransform.GetTransformRootSpace(Transform);
		break;
	case ERigVMTransformSpace::LocalSpace:
		LiveLinkTransform.GetTransformParentSpace(Transform);
		break;
	default:
		checkNoEntry(); // Unknown bone space. We should never hit this
		break;
	}
}

FRigUnit_LiveLinkGetParameterValueByName_Execute()
{
	float NewValue;
	if (SubjectFrame.GetCurveValueByName(ParameterName, NewValue))
	{
		Value = NewValue;
	}
	else
	{
		Value = 0.0f;
	}
}

FRigUnit_LiveLinkEvaluteFrameTransform_Execute()
{
	// Ensure a default value in case the evaluation fails
	Transform = FTransform::Identity;

	// Try to update Transform from Live Link
	if (ILiveLinkClient* LiveLinkClient = LiveLinkControlRigUtilities::TryGetLiveLinkClient())
	{
		FLiveLinkSubjectFrameData FrameData;
		if (LiveLinkClient->EvaluateFrame_AnyThread(SubjectName, ULiveLinkTransformRole::StaticClass(), FrameData))
		{
			FLiveLinkTransformFrameData* TransformData = FrameData.FrameData.Cast<FLiveLinkTransformFrameData>();
			Transform = TransformData->Transform;
		}
	}

	// Debug Drawing if required
	if (bDrawDebug && ExecuteContext.GetDrawInterface() != nullptr)
	{
		// Rather than drawing as an axis guide, draw as a hierarchy with a dummy root bone which makes it clearer what the origin of the transform
		// space is

		/*
		FRigBoneHierarchy DebugHierarchy;

		FName RootName("root");
		FName TransformName("transform");
		DebugHierarchy.Add(RootName, FName(NAME_None), ERigBoneType::User, FTransform::Identity, FTransform::Identity, FTransform::Identity);
		DebugHierarchy.Add(TransformName, RootName, ERigBoneType::User, Transform, Transform, Transform);

		Context.DrawInterface->DrawHierarchy(DebugDrawOffset, DebugHierarchy, EControlRigDrawHierarchyMode::Axes, 1.0f, DebugColor, 1.0f);
		*/
	}
}

FRigUnit_LiveLinkEvaluateBasicValue_Execute()
{
	// Get value by property name from basic data struct
	if (ILiveLinkClient* LiveLinkClient = LiveLinkControlRigUtilities::TryGetLiveLinkClient())
	{
		FLiveLinkSubjectFrameData FrameData;
		if (LiveLinkClient->EvaluateFrame_AnyThread(SubjectName, ULiveLinkBasicRole::StaticClass(), FrameData))
		{
			const FLiveLinkBaseStaticData* StaticData = FrameData.StaticData.Cast<FLiveLinkBaseStaticData>();
			if(StaticData)
			{
				const int32 FoundIndex = StaticData->PropertyNames.Find(PropertyName);

				if(FoundIndex != INDEX_NONE)
				{
					FLiveLinkBaseFrameData* BasicData = FrameData.FrameData.Cast<FLiveLinkBaseFrameData>();
					if(StaticData->PropertyNames.Num() == BasicData->PropertyValues.Num()) //Catch case where the number of PropertyNames and PropertyValues don't match. This can happen in Blueprint Virtual Subjects.
					{
					Value = BasicData->PropertyValues[FoundIndex];
					}
				}
			}
		}
	}
}
