// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlRigDefines.h"
#include "LiveLinkTypes.h"
#include "Roles/LiveLinkInputDeviceTypes.h"
#include "Units/RigUnit.h"

#include "LiveLinkInputDeviceRigUnits.generated.h"

#define UE_API LIVELINKCONTROLRIG_API

// A node to evaluate a live link input device value
USTRUCT(meta = (DisplayName = "Get Live Link Input Device Data", Category = "Live Link"))
struct FRigUnit_LiveLinkEvaluateInputDeviceValue : public FRigUnit
{
	GENERATED_BODY()

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** Name of the subject we would like to get Live Link device data. This should be the subject attached to the gamepad. */
	UPROPERTY(meta = (Input))
	FName SubjectName;

	/** Gamepad device data for the current frame. */
	UPROPERTY(meta = (Output))
	FLiveLinkGamepadInputDeviceFrameData InputDeviceData;
};

#undef UE_API
