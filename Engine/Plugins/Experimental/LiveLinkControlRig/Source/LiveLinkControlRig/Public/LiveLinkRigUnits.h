// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "ControlRigDefines.h"
#include "Units/RigUnit.h"
#include "Roles/LiveLinkAnimationBlueprintStructs.h"
#include "LiveLinkTypes.h"
#include "LiveLinkRigUnits.generated.h"

#define UE_API LIVELINKCONTROLRIG_API

namespace LiveLinkControlRigUtilities
{
	ILiveLinkClient* TryGetLiveLinkClient();
}

/*
 * The base class for all live link control rig nodes
 */
USTRUCT(meta = (Abstract, NodeColor = "0.3 0.1 0.1", DocumentationPolicy = "Strict"))
struct FRigUnit_LiveLinkBase : public FRigUnit
{
	GENERATED_BODY()
};

/**
 * Evaluate current Live Link Animation associated with supplied subject
 */
USTRUCT(meta = (DisplayName = "Evaluate Live Link Frame (Animation)", Category = "Live Link"))
struct FRigUnit_LiveLinkEvaluteFrameAnimation : public FRigUnit_LiveLinkBase
{
	GENERATED_BODY()

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The name of the subject to evaluate a frame for
	UPROPERTY(meta = (Input))
	FName SubjectName;

	// If True debug data will be drawn for the subject
	UPROPERTY(meta = (Input))
	bool bDrawDebug = false;

	// The color to use for the debug drawing
	UPROPERTY(meta = (Input))
	FLinearColor DebugColor = FLinearColor::Red;

	// The world offset to use when drawing the debug data
	UPROPERTY(meta = (Input))
	FTransform DebugDrawOffset;

	// The resulting subject's frame
	UPROPERTY(meta = (Output))
	FSubjectFrameHandle SubjectFrame;
};

/**
 * Get the transform value with supplied subject frame
 */
USTRUCT(meta = (DisplayName = "Get Transform By Name", Category = "Live Link"))
struct FRigUnit_LiveLinkGetTransformByName : public FRigUnit_LiveLinkBase
{
	GENERATED_BODY()

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The frame to receive a transform from
	UPROPERTY(meta = (Input))
	FSubjectFrameHandle SubjectFrame;

	// The name of the transform to retrieve
	UPROPERTY(meta = (Input))
	FName TransformName;

	// The space to retrieve the transform in
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space = ERigVMTransformSpace::LocalSpace;

	// The resulting transform
	UPROPERTY(meta = (Output))
	FTransform Transform;
};

/**
 * Get the parameter value with supplied subject frame 
 */
USTRUCT(meta = (DisplayName = "Get Parameter Value By Name", Category = "Live Link"))
struct FRigUnit_LiveLinkGetParameterValueByName : public FRigUnit_LiveLinkBase
{
	GENERATED_BODY()

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The frame to retrieve the parameter from
	UPROPERTY(meta = (Input))
	FSubjectFrameHandle SubjectFrame;

	// The name of the parameter to retrieve
	UPROPERTY(meta = (Input))
	FName ParameterName;

	// The resulting value of the parameter
	UPROPERTY(meta = (Output))
	float Value = 0.f;
};

/**
 * Evaluate current Live Link Transform associated with supplied subject
 */
USTRUCT(meta = (DisplayName = "Evaluate Live Link Frame (Transform)", Category = "Live Link"))
struct FRigUnit_LiveLinkEvaluteFrameTransform : public FRigUnit_LiveLinkBase
{
	GENERATED_BODY()

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The name of the subject to evaluate
	UPROPERTY(meta = (Input))
	FName SubjectName;

	// If True debug data will be drawn
	UPROPERTY(meta = (Input))
	bool bDrawDebug = false;

	// The color to use for the debug drawing
	UPROPERTY(meta = (Input))
	FLinearColor DebugColor = FLinearColor::Red;

	// The world offset to use for the debug drawing
	UPROPERTY(meta = (Input))
	FTransform DebugDrawOffset;

	// The resulting transform of the subject
	UPROPERTY(meta = (Output))
	FTransform Transform;
};
/**
 * Evaluate current Live Link Basic float property data associated with supplied subject
 */
USTRUCT(meta = (DisplayName = "Get Basic Live Link Data", Category = "Live Link"))
struct FRigUnit_LiveLinkEvaluateBasicValue : public FRigUnit_LiveLinkBase
{
	GENERATED_BODY()

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The name of the subject to evaluate
	UPROPERTY(meta = (Input))
	FName SubjectName;

	// The name of the property to evaluate
	UPROPERTY(meta = (Input))
	FName PropertyName;

	// The resulting value of the evaluated property
	UPROPERTY(meta = (Output))
	float Value = 0.f;
};

#undef UE_API
